{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "FLAC audio decoder for Electra",
	"Description": "Implements FLAC audio playback with the Electra media player",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"Category": "Media Players",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "FlacDecoderElectra",
			"Type": "Runtime",
			"LoadingPhase": "PostEngineInit",
			"PlatformAllowList": [ "Win64" ],
			"TargetDenyList": [ "Server" ]
		}
	],
	"Plugins": [
		{
			"Name": "ElectraCodecs",
			"Enabled": true
		}
	]
}
