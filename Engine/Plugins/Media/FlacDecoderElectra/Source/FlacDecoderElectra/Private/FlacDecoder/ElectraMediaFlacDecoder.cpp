// Copyright Epic Games, Inc. All Rights Reserved.

#include "ElectraMediaFlacDecoder.h"
#include "FlacDecoderElectraModule.h"
#include "IElectraCodecRegistry.h"
#include "IElectraCodecFactory.h"
#include "Misc/ScopeLock.h"
#include "Templates/SharedPointer.h"
#include "Features/IModularFeatures.h"
#include "Features/IModularFeature.h"
#include "IElectraCodecFactoryModule.h"
#include "IElectraDecoderFeaturesAndOptions.h"
#include "IElectraDecoder.h"
#include "IElectraDecoderOutputAudio.h"
#include "ElectraDecodersUtils.h"
#include "MP4Utilities.h"				// for parsing the 'dfLa' box
#include "Utilities/ElectraBitstream.h"
#include "Utils/ElectraBitstreamProcessorDefault.h"

#include "stream_decoder.h"

#define ERRCODE_INTERNAL_NO_ERROR							0
#define ERRCODE_INTERNAL_ALREADY_CLOSED						1
#define ERRCODE_INTERNAL_FAILED_TO_PARSE_CSD				2
#define ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT				3
#define ERRCODE_INTERNAL_UNSUPPORTED_CHANNEL_LAYOUT			4

/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/

class FElectraAudioDecoderOutputFlac_Common : public IElectraDecoderAudioOutput
{
public:
	virtual ~FElectraAudioDecoderOutputFlac_Common()
	{
		FMemory::Free(Buffer);
	}

	FTimespan GetPTS() const override
	{
		return PTS;
	}
	uint64 GetUserValue() const override
	{
		return UserValue;
	}

	int32 GetNumChannels() const override
	{
		return NumChannels;
	}
	int32 GetSampleRate() const override
	{
		return SampleRate;
	}
	int32 GetNumFrames() const override
	{
		return NumFrames;
	}
	bool IsInterleaved() const override
	{
		return true;
	}
	EChannelPosition GetChannelPosition(int32 InChannelNumber) const override
	{
		return InChannelNumber >= 0 && InChannelNumber < ChannelPositions.Num() ? ChannelPositions[InChannelNumber] : EChannelPosition::Invalid;
	}
	ESampleFormat GetSampleFormat() const override
	{
		return ESampleFormat::Float;
	}
	int32 GetBytesPerSample() const override
	{
		return sizeof(float);
	}
	int32 GetBytesPerFrame() const override
	{
		return GetBytesPerSample() * GetNumChannels();
	}
	const void* GetData(int32 InChannelNumber) const override
	{
		return InChannelNumber >= 0 && InChannelNumber < GetNumChannels() ? Buffer + InChannelNumber : nullptr;
	}

public:
	TArray<EChannelPosition> ChannelPositions;
	FTimespan PTS;
	float* Buffer = nullptr;
	uint64 UserValue = 0;
	int32 NumChannels = 0;
	int32 SampleRate = 0;
	int32 NumFrames = 0;
};


class FElectraFlacDecoder : public IElectraDecoder
{
public:
	static void GetConfigurationOptions(TMap<FString, FVariant>& OutOptions)
	{
	}

	FElectraFlacDecoder(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions);
	virtual ~FElectraFlacDecoder();

	IElectraDecoder::EType GetType() const override
	{
		return IElectraDecoder::EType::Audio;
	}

	void GetFeatures(TMap<FString, FVariant>& OutFeatures) const override;

	FError GetError() const override;

	void Close() override;
	IElectraDecoder::ECSDCompatibility IsCompatibleWith(const TMap<FString, FVariant>& CSDAndAdditionalOptions) override;
	bool ResetToCleanStart() override;

	EDecoderError DecodeAccessUnit(const FInputAccessUnit& InInputAccessUnit, const TMap<FString, FVariant>& InAdditionalOptions) override;
	EDecoderError SendEndOfData() override;
	EDecoderError Flush() override;
	EOutputStatus HaveOutput() override;
	TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> GetOutput() override;

	TSharedPtr<IElectraDecoderBitstreamProcessor, ESPMode::ThreadSafe> CreateBitstreamProcessor() override
	{ return FElectraDecoderBitstreamProcessorDefault::Create(); }

	void Suspend() override
	{ }
	void Resume() override
	{ }

private:
	struct FdfLaConfig
	{
		int32 SampleRate = 0;
		int32 NumberOfChannels = 0;
		void Reset()
		{
			SampleRate = 0;
			NumberOfChannels = 0;
		}
		bool SameAs(const FdfLaConfig& rhs)
		{
			return SampleRate == rhs.SampleRate && NumberOfChannels == rhs.NumberOfChannels;
		}
	};

	struct FCurrentInput
	{
		const void* AccessUnit = nullptr;
		int32 RemainingSize = 0;
		bool bAtEOS = false;
		bool bGotEOS = false;
		FLAC__StreamDecoderErrorStatus DecoderError = FLAC__STREAM_DECODER_ERROR_STATUS_UNPARSEABLE_STREAM;
		void Reset()
		{
			AccessUnit = nullptr;
			RemainingSize = 0;
			bAtEOS = false;
			bGotEOS = false;
			DecoderError = FLAC__STREAM_DECODER_ERROR_STATUS_UNPARSEABLE_STREAM;
		}
	};

	bool Parse_dfLa(FdfLaConfig& OutConfig, const TArray<uint8>& IndfLaBox, bool bFailOnError, bool bFullBox);

	bool PostError(int32 ApiReturnValue, FString Message, int32 Code);

	bool InternalDecoderCreate();
	void InternalDecoderDestroy();

	bool SetupChannelMap();
	bool ProcessInput(const void* InData, int64 InDataSize, bool bGetResiduals);

	FLAC__StreamDecoderReadStatus ReadCallback(const FLAC__StreamDecoder* InDecoder, FLAC__byte InBuffer[], size_t* InBytes);
	FLAC__StreamDecoderWriteStatus WriteCallback(const FLAC__StreamDecoder* InDecoder, const FLAC__Frame* InFrame, const FLAC__int32* const InBuffer[]);
	void MetadataCallback(const FLAC__StreamDecoder* InDecoder, const FLAC__StreamMetadata* InMetadata);
	void ErrorCallback(const FLAC__StreamDecoder* InDecoder, FLAC__StreamDecoderErrorStatus InStatus);
	static FLAC__StreamDecoderReadStatus _ReadCallback(const FLAC__StreamDecoder* InDecoder, FLAC__byte InBuffer[], size_t* InBytes, void* InClient_data)
	{ return reinterpret_cast<FElectraFlacDecoder*>(InClient_data)->ReadCallback(InDecoder, InBuffer, InBytes); }
	static FLAC__StreamDecoderWriteStatus _WriteCallback(const FLAC__StreamDecoder* InDecoder, const FLAC__Frame* InFrame, const FLAC__int32* const InBuffer[], void* InClient_data)
	{ return reinterpret_cast<FElectraFlacDecoder*>(InClient_data)->WriteCallback(InDecoder, InFrame, InBuffer); }
	static void _MetadataCallback(const FLAC__StreamDecoder* InDecoder, const FLAC__StreamMetadata* InMetadata, void* InClient_data)
	{ reinterpret_cast<FElectraFlacDecoder*>(InClient_data)->MetadataCallback(InDecoder, InMetadata); }
	static void _ErrorCallback(const FLAC__StreamDecoder* InDecoder, FLAC__StreamDecoderErrorStatus InStatus, void* InClient_data)
	{ reinterpret_cast<FElectraFlacDecoder*>(InClient_data)->ErrorCallback(InDecoder, InStatus); }


	IElectraDecoder::FError LastError;

	FLAC__StreamDecoder* DecoderHandle = nullptr;
	FCurrentInput CurrentInput;

	TSharedPtr<FElectraAudioDecoderOutputFlac_Common, ESPMode::ThreadSafe> CurrentOutput;
	bool bFlushPending = false;

	// Input configuration
	FdfLaConfig DfLaConfig;
	bool bHaveParseddfLa = false;

	// Output
	TArray<IElectraDecoderAudioOutput::EChannelPosition> OutputChannelMap;
};


/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/

class FElectraCommonAudioFlacDecoderFactory : public IElectraCodecFactory, public IElectraCodecFactory::IProviderInformation, public IElectraCodecModularFeature, public TSharedFromThis<FElectraCommonAudioFlacDecoderFactory, ESPMode::ThreadSafe>
{
public:
	virtual ~FElectraCommonAudioFlacDecoderFactory()
	{ }

	FElectraCommonAudioFlacDecoderFactory()
	{
		LibName = ANSI_TO_TCHAR(FLAC__VENDOR_STRING);
		LibVersion = ANSI_TO_TCHAR(FLAC__VERSION_STRING);
	}

	void GetListOfFactories(TArray<TWeakPtr<IElectraCodecFactory, ESPMode::ThreadSafe>>& OutCodecFactories) override
	{
		OutCodecFactories.Add(AsShared());
	}

	void GetConfigurationOptions(TMap<FString, FVariant>& OutOptions) const override
	{
		FElectraFlacDecoder::GetConfigurationOptions(OutOptions);
	}

	int32 SupportsDecoding(TMap<FString, FVariant>& OutFormatInfo, const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions) const override
	{
		if (InCodecFormat.Type != Electra::FCodecTypeFormat::EType::Audio || !InCodecFormat.Properties.IsType<Electra::FCodecTypeFormat::FAudio>())
		{
			return 0;
		}
		if (!InCodecFormat.RFC6381.Equals(TEXT("flac"), ESearchCase::IgnoreCase))
		{
			return 0;
		}
		if (InCodecFormat.HumanReadableFormatInfo.IsEmpty())
		{
			OutFormatInfo.Emplace(IElectraDecoderFormatInfo::HumanReadableFormatName, FVariant(FString(TEXT("Free Lossless Audio Codec (FLAC)"))));
		}
		return 5;
	}

	TSharedPtr<IElectraDecoder, ESPMode::ThreadSafe> CreateDecoder(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions) override
	{
		return InCodecFormat.Properties.IsType<Electra::FCodecTypeFormat::FAudio>() ? MakeShared<FElectraFlacDecoder, ESPMode::ThreadSafe>(InCodecFormat, InAdditionalOptions) : TSharedPtr<IElectraDecoder, ESPMode::ThreadSafe>();
	}


	const IProviderInformation& GetProviderInformation() const override
	{ return *this; }
	FString GetName() const override
	{ return FString(TEXT("libFLAC")); }
	FString GetVersion() const override
	{ return LibVersion; }
	FString GetImplementation() const override
	{ return LibName; }
	FString GetVendor() const override
	{ return(FString()); }

	FString LibName;
	FString LibVersion;
	static TSharedPtr<FElectraCommonAudioFlacDecoderFactory, ESPMode::ThreadSafe> Self;
};
TSharedPtr<FElectraCommonAudioFlacDecoderFactory, ESPMode::ThreadSafe> FElectraCommonAudioFlacDecoderFactory::Self;

/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/

void FElectraMediaFlacDecoder::Startup()
{
	// Make sure the codec factory module has been loaded.
	FModuleManager::Get().LoadModule(TEXT("ElectraCodecFactory"));

	// Create an instance of the factory, which is also the modular feature.
	check(!FElectraCommonAudioFlacDecoderFactory::Self.IsValid());
	FElectraCommonAudioFlacDecoderFactory::Self = MakeShared<FElectraCommonAudioFlacDecoderFactory, ESPMode::ThreadSafe>();
	// Register as modular feature.
	IModularFeatures::Get().RegisterModularFeature(IElectraCodecFactoryModule::GetModularFeatureName(), FElectraCommonAudioFlacDecoderFactory::Self.Get());
}

void FElectraMediaFlacDecoder::Shutdown()
{
	IModularFeatures::Get().UnregisterModularFeature(IElectraCodecFactoryModule::GetModularFeatureName(), FElectraCommonAudioFlacDecoderFactory::Self.Get());
	FElectraCommonAudioFlacDecoderFactory::Self.Reset();
}

TSharedPtr<IElectraCodecFactory, ESPMode::ThreadSafe> FElectraMediaFlacDecoder::CreateFactory()
{
	return MakeShared<FElectraCommonAudioFlacDecoderFactory, ESPMode::ThreadSafe>();;
}

/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/

FElectraFlacDecoder::FElectraFlacDecoder(const Electra::FCodecTypeFormat& InCodecFormat, const TMap<FString, FVariant>& InAdditionalOptions)
{
	if (InCodecFormat.DCR.Num())
	{
		bHaveParseddfLa = Parse_dfLa(DfLaConfig, InCodecFormat.DCR, false, false);
	}
	if (!bHaveParseddfLa && InCodecFormat.ExtraBoxes.Contains(MP4Utilities::MakeBoxAtom('d','f','L','a')))
	{
		bHaveParseddfLa = Parse_dfLa(DfLaConfig, InCodecFormat.ExtraBoxes[MP4Utilities::MakeBoxAtom('d','f','L','a')], false, true);
	}
}

FElectraFlacDecoder::~FElectraFlacDecoder()
{
	// Close() must have been called already!
	check(LastError.Code == ERRCODE_INTERNAL_ALREADY_CLOSED);
	// We do it nonetheless...
	Close();
}

bool FElectraFlacDecoder::Parse_dfLa(FdfLaConfig& OutConfig, const TArray<uint8>& IndfLaBox, bool bFailOnError, bool bFullBox)
{
	if (IndfLaBox.Num() == 0)
	{
		return !bFailOnError ? false : PostError(0, TEXT("There is no 'dfLa' box to get FLAC information from"), ERRCODE_INTERNAL_FAILED_TO_PARSE_CSD);
	}
	else if (IndfLaBox.Num() < 8)
	{
		return !bFailOnError ? false : PostError(0, TEXT("Incomplete 'dfLa' box"), ERRCODE_INTERNAL_FAILED_TO_PARSE_CSD);
	}

	MP4Utilities::FMP4AtomReader rd(IndfLaBox);
	uint8 Value8 = 0;
	if (bFullBox)
	{
		rd.Read(Value8);		// Version
		if (Value8 != 0)
		{
			return !bFailOnError ? false : PostError(0, TEXT("Unsupported 'dfLa' box version"), ERRCODE_INTERNAL_FAILED_TO_PARSE_CSD);
		}
		uint64 Flags = 0;
		rd.ReadAsNumber(Flags, 3);
		if (Flags != 0)
		{
			return !bFailOnError ? false : PostError(0, TEXT("Unsupported 'dfLa' box flags"), ERRCODE_INTERNAL_FAILED_TO_PARSE_CSD);
		}
	}

	int32 LenToGo = IndfLaBox.Num() - (bFullBox ? 4 : 0);
	while(1)
	{
		if (LenToGo < 4)
		{
			return !bFailOnError ? false : PostError(0, TEXT("Incomplete 'dfLa' box"), ERRCODE_INTERNAL_FAILED_TO_PARSE_CSD);
		}
		// Last block and type indicator.
		rd.Read(Value8);
		bool bIsLast = (Value8 & 0x80) != 0;
		uint8 BlockType = Value8 & 0x7f;
		uint64 l;
		rd.ReadAsNumber(l, 3);
		int32 Length = (int32)l;
		TArray<uint8> BlockData;
		BlockData.AddUninitialized(Length);
		LenToGo -= 4;
		if (LenToGo < Length)
		{
			return !bFailOnError ? false : PostError(0, TEXT("Incomplete 'dfLa' box"), ERRCODE_INTERNAL_FAILED_TO_PARSE_CSD);
		}
		for(int32 i=0; i<Length; ++i)
		{
			rd.Read(BlockData[i]);
		}
		LenToGo -= Length;

		// STREAMINFO block?
		if (BlockType == 0)
		{
			Electra::FBitstreamReader si(BlockData.GetData(), BlockData.Num());
			uint16 MinimumBlockSize = si.GetBits(16);
			uint16 MaximumBlockSize = si.GetBits(16);
			uint32 MinimumFrameSize = si.GetBits(24);
			uint32 MaximumFrameSize = si.GetBits(24);
			uint32 SampleRate = si.GetBits(20);
			uint32 NumberOfChannels = si.GetBits(3) + 1;
			uint32 BitsPerSample = si.GetBits(5) + 1;
			uint64 TotalSamples = (((uint64)si.GetBits(4)) << 32) + si.GetBits(32);
			// Remaining 128 bits are MD5 signature that we do not need.

			OutConfig.SampleRate = (int32)SampleRate;
			OutConfig.NumberOfChannels = (int32)NumberOfChannels;
		}

		if (bIsLast || LenToGo <= 0)
		{
			break;
		}
	}

	return true;
}



bool FElectraFlacDecoder::InternalDecoderCreate()
{
	if (!DecoderHandle)
	{
		DecoderHandle = FLAC__stream_decoder_new();
		if (!DecoderHandle)
		{
			return PostError(0, TEXT("FLAC__stream_decoder_new() failed"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
		}
		// Note: If any FLAC__stream_decoder_set_XXX() call needs to be made, it needs to be called here before calling the initialization method.

		FLAC__StreamDecoderInitStatus Result = FLAC__stream_decoder_init_stream(DecoderHandle, &FElectraFlacDecoder::_ReadCallback, nullptr, nullptr, nullptr, nullptr, &FElectraFlacDecoder::_WriteCallback, &FElectraFlacDecoder::_MetadataCallback, &FElectraFlacDecoder::_ErrorCallback, this);
		if (Result != FLAC__STREAM_DECODER_INIT_STATUS_OK)
		{
			return PostError(Result, TEXT("FLAC__stream_decoder_init_stream() failed"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
		}
	}
	return true;
}

void FElectraFlacDecoder::InternalDecoderDestroy()
{
	if (DecoderHandle)
	{
		FLAC__stream_decoder_finish(DecoderHandle);
		FLAC__stream_decoder_delete(DecoderHandle);
		DecoderHandle = nullptr;
	}
}


void FElectraFlacDecoder::GetFeatures(TMap<FString, FVariant>& OutFeatures) const
{
	GetConfigurationOptions(OutFeatures);
}

IElectraDecoder::FError FElectraFlacDecoder::GetError() const
{
	return LastError;
}

bool FElectraFlacDecoder::PostError(int32 ApiReturnValue, FString Message, int32 Code)
{
	LastError.Code = Code;
	LastError.SdkCode = ApiReturnValue;
	LastError.Message = MoveTemp(Message);
	return false;
}

void FElectraFlacDecoder::Close()
{
	ResetToCleanStart();
	// Set the error state that all subsequent calls will fail.
	PostError(0, TEXT("Already closed"), ERRCODE_INTERNAL_ALREADY_CLOSED);
}

IElectraDecoder::ECSDCompatibility FElectraFlacDecoder::IsCompatibleWith(const TMap<FString, FVariant>& CSDAndAdditionalOptions)
{
	// No configuration parsed yet, so this is deemed compatible.
	if (!bHaveParseddfLa)
	{
		return IElectraDecoder::ECSDCompatibility::Compatible;
	}
	TArray<uint8> DCR = ElectraDecodersUtil::GetVariantValueUInt8Array(CSDAndAdditionalOptions, TEXT("dcr"));
	TArray<uint8> CSD = ElectraDecodersUtil::GetVariantValueUInt8Array(CSDAndAdditionalOptions, TEXT("csd"));
	FdfLaConfig cfg;
	if (!Parse_dfLa(cfg, DCR.Num() ? DCR : CSD, false, false))
	{
		return IElectraDecoder::ECSDCompatibility::DrainAndReset;
	}
	if (cfg.SameAs(DfLaConfig))
	{
		return IElectraDecoder::ECSDCompatibility::Compatible;
	}
	return IElectraDecoder::ECSDCompatibility::DrainAndReset;
}

bool FElectraFlacDecoder::ResetToCleanStart()
{
	bFlushPending = false;
	CurrentOutput.Reset();

	bHaveParseddfLa = false;
	DfLaConfig.Reset();
	OutputChannelMap.Empty();
	InternalDecoderDestroy();
	return true;
}

bool FElectraFlacDecoder::SetupChannelMap()
{
	if (OutputChannelMap.Num())
	{
		return true;
	}
	// Pre-init with all channels disabled.
	OutputChannelMap.Empty();
	OutputChannelMap.Init(IElectraDecoderAudioOutput::EChannelPosition::Disabled, DfLaConfig.NumberOfChannels);
	if (DfLaConfig.NumberOfChannels == 1)
	{
		OutputChannelMap[0] = IElectraDecoderAudioOutput::EChannelPosition::C;
	}
	else if (DfLaConfig.NumberOfChannels == 2)
	{
		OutputChannelMap[0] = IElectraDecoderAudioOutput::EChannelPosition::L;
		OutputChannelMap[1] = IElectraDecoderAudioOutput::EChannelPosition::R;
	}
	else if (DfLaConfig.NumberOfChannels == 3)
	{
		OutputChannelMap[0] = IElectraDecoderAudioOutput::EChannelPosition::L;
		OutputChannelMap[1] = IElectraDecoderAudioOutput::EChannelPosition::R;
		OutputChannelMap[2] = IElectraDecoderAudioOutput::EChannelPosition::C;
	}
	else if (DfLaConfig.NumberOfChannels == 4)
	{
		OutputChannelMap[0] = IElectraDecoderAudioOutput::EChannelPosition::L;
		OutputChannelMap[1] = IElectraDecoderAudioOutput::EChannelPosition::R;
		OutputChannelMap[2] = IElectraDecoderAudioOutput::EChannelPosition::Ls;
		OutputChannelMap[3] = IElectraDecoderAudioOutput::EChannelPosition::Rs;
	}
	else if (DfLaConfig.NumberOfChannels == 5)
	{
		OutputChannelMap[0] = IElectraDecoderAudioOutput::EChannelPosition::L;
		OutputChannelMap[1] = IElectraDecoderAudioOutput::EChannelPosition::R;
		OutputChannelMap[2] = IElectraDecoderAudioOutput::EChannelPosition::C;
		OutputChannelMap[3] = IElectraDecoderAudioOutput::EChannelPosition::Ls;
		OutputChannelMap[4] = IElectraDecoderAudioOutput::EChannelPosition::Rs;
	}
	else if (DfLaConfig.NumberOfChannels == 6)
	{
		OutputChannelMap[0] = IElectraDecoderAudioOutput::EChannelPosition::L;
		OutputChannelMap[1] = IElectraDecoderAudioOutput::EChannelPosition::R;
		OutputChannelMap[2] = IElectraDecoderAudioOutput::EChannelPosition::C;
		OutputChannelMap[3] = IElectraDecoderAudioOutput::EChannelPosition::LFE;
		OutputChannelMap[4] = IElectraDecoderAudioOutput::EChannelPosition::Ls;
		OutputChannelMap[5] = IElectraDecoderAudioOutput::EChannelPosition::Rs;
	}
	else if (DfLaConfig.NumberOfChannels == 7)
	{
		OutputChannelMap[0] = IElectraDecoderAudioOutput::EChannelPosition::L;
		OutputChannelMap[1] = IElectraDecoderAudioOutput::EChannelPosition::R;
		OutputChannelMap[2] = IElectraDecoderAudioOutput::EChannelPosition::C;
		OutputChannelMap[3] = IElectraDecoderAudioOutput::EChannelPosition::LFE;
		OutputChannelMap[4] = IElectraDecoderAudioOutput::EChannelPosition::Cs;
		OutputChannelMap[5] = IElectraDecoderAudioOutput::EChannelPosition::Ls;
		OutputChannelMap[6] = IElectraDecoderAudioOutput::EChannelPosition::Rs;
	}
	else // DfLaConfig.NumberOfChannels == 8
	{
		OutputChannelMap[0] = IElectraDecoderAudioOutput::EChannelPosition::L;
		OutputChannelMap[1] = IElectraDecoderAudioOutput::EChannelPosition::R;
		OutputChannelMap[2] = IElectraDecoderAudioOutput::EChannelPosition::C;
		OutputChannelMap[3] = IElectraDecoderAudioOutput::EChannelPosition::LFE;
		OutputChannelMap[4] = IElectraDecoderAudioOutput::EChannelPosition::Ls;
		OutputChannelMap[5] = IElectraDecoderAudioOutput::EChannelPosition::Rs;
		OutputChannelMap[6] = IElectraDecoderAudioOutput::EChannelPosition::Lsr;
		OutputChannelMap[7] = IElectraDecoderAudioOutput::EChannelPosition::Rsr;
	}

	return true;
}


FLAC__StreamDecoderReadStatus FElectraFlacDecoder::ReadCallback(const FLAC__StreamDecoder* InDecoder, FLAC__byte InBuffer[], size_t* InBytes)
{
	if (InDecoder == DecoderHandle)
	{
		if (CurrentInput.RemainingSize <= 0)
		{
			*InBytes = 0;
			return CurrentInput.bAtEOS ? FLAC__STREAM_DECODER_READ_STATUS_END_OF_STREAM : FLAC__STREAM_DECODER_READ_STATUS_ABORT;
		}
		int32 nb = (int32)(*InBytes) < CurrentInput.RemainingSize ? (int32)(*InBytes) : CurrentInput.RemainingSize;
		FMemory::Memcpy(InBuffer, CurrentInput.AccessUnit, nb);
		CurrentInput.RemainingSize -= nb;
		CurrentInput.AccessUnit = (const void*)(UPTRINT(CurrentInput.AccessUnit) + UPTRINT(nb));
		*InBytes = nb;
		return FLAC__STREAM_DECODER_READ_STATUS_CONTINUE;
	}
	return FLAC__STREAM_DECODER_READ_STATUS_ABORT;
}

FLAC__StreamDecoderWriteStatus FElectraFlacDecoder::WriteCallback(const FLAC__StreamDecoder* InDecoder, const FLAC__Frame* InFrame, const FLAC__int32* const InBuffer[])
{
	if (InFrame->header.blocksize == 0)
	{
		CurrentInput.bGotEOS = true;
		return FLAC__STREAM_DECODER_WRITE_STATUS_CONTINUE;
	}

	if (InFrame->header.channels != DfLaConfig.NumberOfChannels)
	{
		PostError(0, TEXT("Mismatching number of channels between CSD and actual decoded output!"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
		return FLAC__STREAM_DECODER_WRITE_STATUS_ABORT;
	}

	// First call?
	if (CurrentOutput->NumFrames == 0)
	{
		uint64 AllocSize = sizeof(float) * InFrame->header.blocksize * InFrame->header.channels;
		CurrentOutput->Buffer = (float*)FMemory::Malloc(AllocSize);
		CurrentOutput->NumChannels = (int32) InFrame->header.channels;
		CurrentOutput->SampleRate = (int32) InFrame->header.sample_rate;
	}
	else
	{
		if (CurrentOutput->NumChannels != (int32) InFrame->header.channels)
		{
			PostError(0, TEXT("Mismatching number of decoded channels during decode sequence!"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
			return FLAC__STREAM_DECODER_WRITE_STATUS_ABORT;
		}
		if (CurrentOutput->SampleRate != (int32) InFrame->header.sample_rate)
		{
			PostError(0, TEXT("Mismatching sample rate during decode sequence!"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
			return FLAC__STREAM_DECODER_WRITE_STATUS_ABORT;
		}
		uint64 ReAllocSize = sizeof(float) * (InFrame->header.blocksize + CurrentOutput->NumFrames) * InFrame->header.channels;
		CurrentOutput->Buffer = (float*)FMemory::Realloc(CurrentOutput->Buffer, ReAllocSize);
	}
	if (!CurrentOutput->Buffer)
	{
		PostError(0, TEXT("Failed to allocate output sample buffer!"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
		return FLAC__STREAM_DECODER_WRITE_STATUS_ABORT;
	}
	// Get the samples across
	float Scale = (InFrame->header.bits_per_sample >= 8 && InFrame->header.bits_per_sample <= 32) ? 1.0f / (int32)(1 << (InFrame->header.bits_per_sample - 1)) : 0.0;
	for(int32 ch=0; ch<CurrentOutput->NumChannels; ++ch)
	{
		const FLAC__int32* DecodedChannelBase = InBuffer[ch];
		float* OutChannelBase = CurrentOutput->Buffer + (CurrentOutput->NumFrames * CurrentOutput->NumChannels) + ch;
		for(uint32 i=0; i<InFrame->header.blocksize; ++i)
		{
			*OutChannelBase = *DecodedChannelBase++ * Scale;
			OutChannelBase += CurrentOutput->NumChannels;
		}
	}
	CurrentOutput->NumFrames += (int32) InFrame->header.blocksize;

	return FLAC__STREAM_DECODER_WRITE_STATUS_CONTINUE;
}

void FElectraFlacDecoder::MetadataCallback(const FLAC__StreamDecoder* InDecoder, const FLAC__StreamMetadata* InMetadata)
{
	// Don't care at the moment.
}

void FElectraFlacDecoder::ErrorCallback(const FLAC__StreamDecoder* InDecoder, FLAC__StreamDecoderErrorStatus InStatus)
{
	CurrentInput.DecoderError = InStatus;
}


bool FElectraFlacDecoder::ProcessInput(const void* InData, int64 InDataSize, bool bGetResiduals)
{
	if (!DecoderHandle)
	{
		return false;
	}

	if (!bGetResiduals)
	{
		CurrentInput.Reset();
		CurrentInput.AccessUnit = InData;
		CurrentInput.RemainingSize = (int32)InDataSize;
	}
	else
	{
		CurrentInput.bAtEOS = true;
	}

	while(CurrentInput.RemainingSize > 0 || bGetResiduals)
	{
		bool bOk = !!(bGetResiduals ? FLAC__stream_decoder_process_until_end_of_stream(DecoderHandle) : FLAC__stream_decoder_process_single(DecoderHandle));
		if (!bOk)
		{
			return PostError(CurrentInput.DecoderError, TEXT("FLAC__stream_decoder_process_single() failed"), ERRCODE_INTERNAL_FAILED_TO_DECODE_INPUT);
		}
		FLAC__StreamDecoderState State = FLAC__stream_decoder_get_state(DecoderHandle);
		if (State == FLAC__STREAM_DECODER_END_OF_STREAM || State == FLAC__STREAM_DECODER_ABORTED || CurrentInput.bGotEOS)
		{
			// At the end of the stream, flush the decoder so it can be used again.
			if (State == FLAC__STREAM_DECODER_END_OF_STREAM)
			{
				FLAC__stream_decoder_flush(DecoderHandle);
			}
			break;
		}
	}
	CurrentOutput->ChannelPositions = OutputChannelMap;
	return true;
}

IElectraDecoder::EDecoderError FElectraFlacDecoder::DecodeAccessUnit(const FInputAccessUnit& InInputAccessUnit, const TMap<FString, FVariant>& InAdditionalOptions)
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}

	// Can not feed new input until draining has finished.
	if (bFlushPending)
	{
		return IElectraDecoder::EDecoderError::EndOfData;
	}

	// If there is pending output it is very likely that decoding this access unit would also generate output.
	// Since that would result in loss of the pending output we return now.
	if (CurrentOutput.IsValid())
	{
		return IElectraDecoder::EDecoderError::NoBuffer;
	}

	// Decode data.
	if (InInputAccessUnit.Data && InInputAccessUnit.DataSize)
	{
		// Parse the codec specific information
		if (!bHaveParseddfLa)
		{
			if (InAdditionalOptions.Contains(TEXT("dcr")))
			{
				bHaveParseddfLa = Parse_dfLa(DfLaConfig, ElectraDecodersUtil::GetVariantValueUInt8Array(InAdditionalOptions, TEXT("dcr")), true, false);
			}
			if (!bHaveParseddfLa && InAdditionalOptions.Contains(TEXT("dfLa")))
			{
				bHaveParseddfLa = Parse_dfLa(DfLaConfig, ElectraDecodersUtil::GetVariantValueUInt8Array(InAdditionalOptions, TEXT("dfLa")), true, true);
			}
			if (!bHaveParseddfLa)
			{
				return IElectraDecoder::EDecoderError::Error;
			}
		}
		// Set up the channel map accordingly.
		if (!SetupChannelMap())
		{
			// Error was already posted.
			return IElectraDecoder::EDecoderError::Error;
		}
		// Create decoder if necessary.
		if (!DecoderHandle && !InternalDecoderCreate())
		{
			return IElectraDecoder::EDecoderError::Error;
		}
		// Prepare the output
		if (!CurrentOutput.IsValid())
		{
			CurrentOutput = MakeShared<FElectraAudioDecoderOutputFlac_Common>();
			CurrentOutput->PTS = InInputAccessUnit.PTS;
			CurrentOutput->UserValue = InInputAccessUnit.UserValue;
		}
		// Decode
		if (!ProcessInput(InInputAccessUnit.Data, InInputAccessUnit.DataSize, false))
		{
			CurrentOutput.Reset();
			return IElectraDecoder::EDecoderError::Error;
		}
	}
	return IElectraDecoder::EDecoderError::None;
}

IElectraDecoder::EDecoderError FElectraFlacDecoder::SendEndOfData()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}
	// Already draining?
	if (bFlushPending)
	{
		return IElectraDecoder::EDecoderError::EndOfData;
	}
	bFlushPending = true;
	return IElectraDecoder::EDecoderError::None;
}

IElectraDecoder::EDecoderError FElectraFlacDecoder::Flush()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EDecoderError::Error;
	}
	ResetToCleanStart();
	return IElectraDecoder::EDecoderError::None;
}

IElectraDecoder::EOutputStatus FElectraFlacDecoder::HaveOutput()
{
	// If already in error do nothing!
	if (LastError.IsSet())
	{
		return IElectraDecoder::EOutputStatus::Error;
	}
	// Have output?
	if (CurrentOutput.IsValid())
	{
		return IElectraDecoder::EOutputStatus::Available;
	}
	// Pending flush?
	if (bFlushPending)
	{
		if (!CurrentOutput.IsValid())
		{
			CurrentOutput = MakeShared<FElectraAudioDecoderOutputFlac_Common>();
			//CurrentOutput->PTS = InInputAccessUnit.PTS;
			//CurrentOutput->UserValue = InInputAccessUnit.UserValue;
		}
		// Decode residuals
		if (!ProcessInput(nullptr, 0, true))
		{
			CurrentOutput.Reset();
			return IElectraDecoder::EOutputStatus::Error;
		}

		// TBD
		//   We can't actually use any residuals since we did not actually sent input for it
		//   and thus have no PTS or user value for it. Also, since the upper layer will not
		//   expect output for something he did not actually ask for...
		CurrentOutput.Reset();

		bFlushPending = false;
		return IElectraDecoder::EOutputStatus::EndOfData;
	}
	return IElectraDecoder::EOutputStatus::NeedInput;
}

TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> FElectraFlacDecoder::GetOutput()
{
	TSharedPtr<IElectraDecoderOutput, ESPMode::ThreadSafe> Out = CurrentOutput;
	CurrentOutput.Reset();
	return Out;
}
