// Copyright Epic Games, Inc. All Rights Reserved.

#include "FlacDecoderElectraModule.h"
#include "FlacDecoder/ElectraMediaFlacDecoder.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "FlacDecoderElectraModule"

DEFINE_LOG_CATEGORY(LogFlacElectraDecoder);

class FFlacElectraDecoderModule : public IModuleInterface
{
public:
	void StartupModule() override
	{
		FElectraMediaFlacDecoder::Startup();
	}

	void ShutdownModule() override
	{
		FElectraMediaFlacDecoder::Shutdown();
	}

	bool SupportsDynamicReloading() override
	{
		// Codec could still be in use
		return false;
	}
};

IMPLEMENT_MODULE(FFlacElectraDecoderModule, FlacDecoderElectra);

#undef LOCTEXT_NAMESPACE
